{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "InstanceDataObject Fixup Tool",
	"Description" : "Editor tool for redirecting loose InstanceDataObject properties",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"IsExperimentalVersion" : true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "InstanceDataObjectFixupTool",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}