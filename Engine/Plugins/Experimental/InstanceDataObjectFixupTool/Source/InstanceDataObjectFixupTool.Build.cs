// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InstanceDataObjectFixupTool : ModuleRules
	{
		public InstanceDataObjectFixupTool(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"Engine",
					"CoreUObject",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SlateCore",
					"Slate",
					"PropertyEditor", 
					"WorkspaceMenuStructure", 
					"UnrealEd",
					"TedsAlerts",
					"TypedElementFramework",
					"Json",
					"TedsTypeInfo",
					"TedsQueryStack",
					"TedsTableViewer",
					"InputCore",
					"ToolWidgets",
					"TedsOutliner",
					"SceneOutliner",
					"EditorWidgets",
				}
			);
		}
	}
}