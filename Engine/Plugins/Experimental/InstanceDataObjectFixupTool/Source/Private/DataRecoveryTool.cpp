// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRecoveryTool.h"

#include "DataStorage/Features.h"
#include "DetailTreeNode.h"
#include "InstanceDataObjectFixupTool.h"
#include "Elements/Columns/TedsTypeInfoColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SDetailsSplitter.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "TedsAlertColumns.h"
#include "TedsQueryNode.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "TedsQueryStack/Public/TedsRowQueryResultsNode.h"
#include "TedsTableViewer/Public/Widgets/STedsSearchBox.h"
#include "TedsTableViewer/Public/Widgets/STedsTableViewer.h"
#include "UObject/InstanceDataTransforms.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/STedsHierarchyViewer.h"
#include "Serialization/ObjectWriter.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/PropertyBagRepository.h"

#include "DataRecoveryToolStyle.h"

#include "Widgets/Layout/SSeparator.h"

#define USE_OPEN_EXTERNAL_BUTTON 0

#define LOCTEXT_NAMESPACE "DataRecoveryTool"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SDataRecoveryTool::Construct(const FArguments& InArgs)
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");
	const float SmallPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Small");

	SelectedClassPath =  MakeShared<FTopLevelAssetPath>(InArgs._SelectedClassPath.IsSet() ? *InArgs._SelectedClassPath : nullptr);
	StagedTransforms = MakeShared<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>>();
	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				MakeTopBar()
			]
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(NoPadding, SmallPadding, NoPadding, NoPadding)
			[
				SNew(SSplitter)
				+SSplitter::Slot()
				.MinSize(290.f)
				.Value(.3f)
				[
					MakeClassesPanel()
				]
				+ SSplitter::Slot()
				.MinSize(290.f)
				[
					MakeRightPanel()
				]
			]
		];
}

TSharedRef<SWidget> SDataRecoveryTool::MakeTopBar()
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");
	const float SmallPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Small");
	const float NormalPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Normal");
	TSharedRef<SHorizontalBox> TopBarContent = SNew(SHorizontalBox);

	if (UE::FInstanceDataTransforms::Get().IsSerializationEnabled())
	{
		TopBarContent->AddSlot()
			.AutoWidth()
			.Padding(0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(NormalPadding)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SDataRecoveryTool::OnSaveAndApplyClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Save"))
					.ToolTipText(INVTEXT("Save"))
				]
			];

		TopBarContent->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				SNew(SSeparator)
				.Thickness(SmallPadding * .5f)
				.Orientation(Orient_Vertical)
				.SeparatorImage(FAppStyle::Get().GetBrush("Brushes.Background"))
			];
	}

	TopBarContent->AddSlot()
		.FillWidth(1.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(NormalPadding)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SDataRecoveryTool::OnApplyClicked)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("LevelEditor.Recompile.Small"))
					]
					+ SHorizontalBox::Slot()
					.Padding(SmallPadding, NoPadding, NoPadding, NoPadding)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ApplyButtonText", "Apply"))
					]
				]
			]
		+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNullWidget::NullWidget
			]
#if USE_OPEN_EXTERNAL_BUTTON
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(NormalPadding)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.OpenInExternalEditor"))
					]
					+ SHorizontalBox::Slot()
					.Padding(SmallPadding, NoPadding, NoPadding, NoPadding)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("EditInVSCodeButton", "Edit in VS Code"))
					]
				]
			]
#endif // USE_OPEN_EXTERNAL_BUTTON
		];

	return
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(NoPadding)
		[
			TopBarContent
		];
}



TSharedRef<SWidget> SDataRecoveryTool::MakeClassesPanel()
{
	return MakePanel(LOCTEXT("ClassesTitle", "CLASSES"), MakeTEDSTableViewer());
}

TSharedRef<SWidget> SDataRecoveryTool::MakeRightPanel()
{
	return MakePanel(LOCTEXT("MappingsTitle", "MAPPINGS"), MakeDiffSection());
}

TSharedRef<SWidget> SDataRecoveryTool::MakeDiffSection()
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");

	return
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(NoPadding)
		[
			SAssignNew(DiffView, SInstanceDataObjectFixupTool)
			.SelectedClassPath(SelectedClassPath)
			.StagedTransforms(StagedTransforms)
		];
}


FReply SDataRecoveryTool::OnApplyClicked() const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::DataRecoveryTool::Utils;
	// for now just apply the transforms to all instances of every listed class. This may be slow!

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	FScopedSlowTask ApplyAllTask{ static_cast<float>(ReferenceQueryResultsNode->GetRows().Num()), LOCTEXT("ApplyAllSlow","Applying transforms") };
	ApplyAllTask.MakeDialog();

	for (const RowHandle Row : ReferenceQueryResultsNode->GetRows())
	{
		const FTypedElementClassTypeInfoColumn* TypeInfo = Storage->GetColumn<FTypedElementClassTypeInfoColumn>(Row);
		if (!TypeInfo)
		{
			continue;
		}

		if (const UClass* Class = TypeInfo->TypeInfo.Get())
		{
			const FTopLevelAssetPath ClassPath = Class->GetClassPathName();

			ApplyAllTask.EnterProgressFrame(1.f, FText::Format(LOCTEXT("ApplyToSlow", "Applying transforms to {0}"), FText::FromString(ClassPath.ToString())));

			TArray<UObject*> AllInstancesOfClasses;

			uint32 InstanceWithUnknownPropertiesCount = 0;

			GetObjectsOfClass(Class, AllInstancesOfClasses);

			for (UObject* Instance : AllInstancesOfClasses)
			{
				if (UObject* Ido = UE::FPropertyBagRepository::Get().FindInstanceDataObject(Instance))
				{
					if (!Snapshot::IsASnapshot(Instance) && UE::FPropertyBagRepository::Get().RequiresFixup(Instance))
					{
						ApplyTransforms(StagedTransforms, Ido, Instance);

						if (UE::FPropertyBagRepository::Get().RequiresFixup(Instance))
						{
							++InstanceWithUnknownPropertiesCount;
						}
					}
				}
			}

			if (InstanceWithUnknownPropertiesCount == 0)
			{
				Storage->RemoveColumn<FHasUnknownPropertiesTag>(Row);
				Storage->AddColumn<FTypedElementSyncBackToWorldTag>(Row);
			}
			else
			{
				UpdateTedsIdoAlertColumn(Storage, Row, InstanceWithUnknownPropertiesCount);
			}

			DiffView->GenerateDetailsViews();
		}
	}

	return FReply::Handled();
}

FReply SDataRecoveryTool::OnSaveAndApplyClicked() const
{
	// save then apply transforms to all instances of this class
	// TODO: struct transforms wont be applied to instances of that struct in other classes. This will require a reload unfortunately
	for (const TPair<FTopLevelAssetPath, UE::FInstanceDataTransformSet>& TransformSet : *StagedTransforms)
	{
		UE::FInstanceDataTransforms::Get().SaveTransformSet(TransformSet.Value);
	}

	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	// for now just apply the transforms to all instances of every listed class. This may be slow!

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	FScopedSlowTask PatchAllTask{ static_cast<float>(ReferenceQueryResultsNode->GetRows().Num()), LOCTEXT("PatchAllSlow","Patching transforms") };
	PatchAllTask.MakeDialog();

	for (const RowHandle Row : ReferenceQueryResultsNode->GetRows())
	{
		const FTypedElementClassTypeInfoColumn* TypeInfo = Storage->GetColumn<FTypedElementClassTypeInfoColumn>(Row);
		if (!TypeInfo)
		{
			continue;
		}

		if (const UClass* Class = TypeInfo->TypeInfo.Get())
		{
			const FTopLevelAssetPath ClassPath = Class->GetClassPathName();

			PatchAllTask.EnterProgressFrame(1.f, FText::Format(LOCTEXT("PatchToSlow", "Patching transforms to {0}"), FText::FromString(ClassPath.ToString())));

			const UE::FInstanceDataTransformSet* TransformSet = StagedTransforms->Find(ClassPath);

			if (TransformSet == nullptr)
			{
				continue;
			}

			TArray<UObject*> AllInstancesOfClasses;

			uint32 InstanceWithUnknownPropertiesCount = 0;

			// Could not use PatchTransformSet inside ForEachObjectOfClass lambda, so we have to get the objects first
			GetObjectsOfClass(Class, AllInstancesOfClasses);

			for (UObject* Instance : AllInstancesOfClasses)
			{
				if (UObject* Ido = UE::FPropertyBagRepository::Get().FindInstanceDataObject(Instance))
				{
					if (!UE::DataRecoveryTool::Utils::Snapshot::IsASnapshot(Instance) && UE::FPropertyBagRepository::Get().RequiresFixup(Instance))
					{
						UE::FInstanceDataTransforms::Get().PatchInstanceDataObject(Ido, Instance);

						if (UE::FPropertyBagRepository::Get().RequiresFixup(Instance))
						{
							++InstanceWithUnknownPropertiesCount;
						}
					}
				}
			}

			if (InstanceWithUnknownPropertiesCount == 0)
			{
				Storage->RemoveColumn<FHasUnknownPropertiesTag>(Row);
				Storage->AddColumn<FTypedElementSyncBackToWorldTag>(Row);
			}
			else
			{
				UE::DataRecoveryTool::Utils::UpdateTedsIdoAlertColumn(Storage, Row, InstanceWithUnknownPropertiesCount);
			}

			DiffView->GenerateDetailsViews();
		}
	}

	return OnApplyClicked();
}

TSharedRef<SWidget> SDataRecoveryTool::MakePanel(const FText& InTitle, const TSharedRef<SWidget>& InContent, const TOptional<FOnGetContent>& InMenuContent)
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");
	const float BigPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Big");

	TSharedRef<SHorizontalBox> Header =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(InTitle)
			.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
		];

	if (InMenuContent.IsSet())
	{
		Header
		->AddSlot()
		.FillWidth(1.f)
		[
			SNullWidget::NullWidget
		];

		Header
		->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SPositiveActionButton)
			.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			.Text(LOCTEXT("AddMappingButton", "Add"))
			.OnGetMenuContent(InMenuContent.GetValue())
		];
	}

	TSharedRef<SWidget> HeaderTitle =
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(BigPadding)
		[
			Header
		];

	return
		SNew(SBorder)
		.Padding(NoPadding)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				HeaderTitle
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.FillContentHeight(1.f)
			[
				InContent
			]
		];
}

void SDataRecoveryTool::OnListSelectionChanged(UE::Editor::DataStorage::RowHandle RowHandle, ESelectInfo::Type Info)
{
	using namespace UE::Editor::DataStorage;

	if (RowHandle == 0)
	{
		*SelectedClassPath = nullptr;

		if (DiffView != nullptr)
		{
			DiffView->GenerateDetailsViews();
		}

		return;
	}

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	FTypedElementClassTypeInfoColumn* TypeColumn = Storage->GetColumn<FTypedElementClassTypeInfoColumn>(RowHandle);

	if (!ensureMsgf(TypeColumn, TEXT("Should have been able to find a row")))
	{
		return;
	}

	const UClass* Type = TypeColumn->TypeInfo.Get();
	if (!ensureMsgf(Type, TEXT("Should have been able to find the Type UClass")))
	{
		return;
	}

	*SelectedClassPath = Type->GetClassPathName();

	if (DiffView == nullptr)
	{
		return;
	}

	DiffView->GenerateDetailsViews();
}

void SDataRecoveryTool::SetSelectedClass()
{
	if (!ClassesTableViewer)
	{
		return;
	}
	FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateLambda([this](float DeltaTime) {
			using namespace UE::Editor::DataStorage;
			ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

			const UClass* LoadedClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, SelectedClassPath->ToString()));
			for (RowHandle RowHandle : ReferenceQueryResultsNode->GetRows())
			{
				FTypedElementClassTypeInfoColumn* TypeInfo = Storage->GetColumn<FTypedElementClassTypeInfoColumn>(RowHandle);
				if (TypeInfo && TypeInfo->TypeInfo == LoadedClass)
				{
					ClassesTableViewer->SetSelection(RowHandle, true, ESelectInfo::Direct);
					break;
				}
			}
			return false;
			}),
		0.0f
	);
}

TSharedRef<SWidget> SDataRecoveryTool::MakeTEDSTableViewer()
{
	const float SmallPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Small");

	using namespace UE::Editor::DataStorage;

	ICoreProvider* Storage = UE::Editor::DataStorage::GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (!ensureMsgf(Storage, TEXT("Cannot create a TEDS ObjectReferenceContext widget before TEDS is initialized")))
	{
		return
			SNew(STextBlock)
			.Text(LOCTEXT("ObjectReferenceErrorCheck", "No valid editor data storage available."));
	}

	using namespace UE::Editor::DataStorage::Queries;

	TSharedRef<QueryStack::FQueryNode> ReferenceQueryNode =
		MakeShared<QueryStack::FQueryNode>(*Storage,
			Select()
				.ReadOnly<FTypedElementLabelColumn>()
			.Where()
				.All<FHasUnknownPropertiesTag, FTypedElementClassTypeInfoColumn>().Compile());

	TSharedPtr<QueryStack::IRowNode> SearchNode;
	TSharedPtr<SBox> SearchBoxContainer = SNew(SBox);

	SearchBoxContainer->SetContent(SNew(UE::Editor::DataStorage::STedsSearchBox)
		.InSearchableQueryNode(ReferenceQueryNode)
		.InSearchableQueryFlags(QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate)
		.OutSearchNode(&SearchNode));

	// If there is no SearchNode available, either disabled or there is an internal error, assign it to be the 
	// ReferenceQueryResultsNode so that it won't be null
	if (!SearchNode)
	{
		ReferenceQueryResultsNode = MakeShared<QueryStack::FRowQueryResultsNode>(
			*Storage,
			ReferenceQueryNode,
			QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
	}
	else
	{
		ReferenceQueryResultsNode = SearchNode;
	}

	// Using the Teds Outliner Purpose for matching customization for now
	IUiProvider::FPurposeID PurposeId = IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", NAME_None).GeneratePurposeID();

	ClassesTableViewer = SNew(STedsTableViewer)
		.QueryStack(ReferenceQueryResultsNode)
		.Columns({ FTypedElementClassTypeInfoColumn::StaticStruct(), FTedsAlertColumn::StaticStruct() })
		.ListSelectionMode(ESelectionMode::Single)
		.OnSelectionChanged(this, &SDataRecoveryTool::OnListSelectionChanged);

	if (SelectedClassPath.IsValid())
	{
		SetSelectedClass();
	}

	return
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(SmallPadding)
		[
			SearchBoxContainer.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		[
			ClassesTableViewer.ToSharedRef()
		];
}

void SDataRecoveryTool::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
}

void SDataRecoveryTool::SetDockTab(const TSharedRef<SDockTab>& DockTab)
{
	OwningDockTab = DockTab;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE

#undef USE_OPEN_EXTERNAL_BUTTON
#undef USE_SAVE_BUTTON
