// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRecoveryToolStyle.h"

#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"

TSharedPtr<FDataRecoveryToolStyle> FDataRecoveryToolStyle::DataRecoveryToolStyle = nullptr;

void FDataRecoveryToolStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FDataRecoveryToolStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

void FDataRecoveryToolStyle::Shutdown()
{
	Unregister();
	DataRecoveryToolStyle.Reset();
}

const FDataRecoveryToolStyle& FDataRecoveryToolStyle::Get()
{
	if (!DataRecoveryToolStyle.IsValid())
	{
		DataRecoveryToolStyle = MakeShareable(new FDataRecoveryToolStyle());
	}

	return *DataRecoveryToolStyle;
}

void FDataRecoveryToolStyle::ReinitializeStyle()
{
	Unregister();
	DataRecoveryToolStyle.Reset();
	Register();
}

void FDataRecoveryToolStyle::InitPadding()
{
	Set("DataRecoveryTool.Padding.None", 0.f);
	Set("DataRecoveryTool.Padding.Small", 4.f);
	Set("DataRecoveryTool.Padding.Normal", 8.f);
	Set("DataRecoveryTool.Padding.Big", 8.f);
}

FDataRecoveryToolStyle::FDataRecoveryToolStyle() : FSlateStyleSet("DataRecoveryTool")
{
	InitPadding();
}

