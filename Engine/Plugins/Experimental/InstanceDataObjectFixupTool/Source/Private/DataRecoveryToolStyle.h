// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateStyle.h"


/** Manages the style which provides resources for data recovery tool widgets. */
class FDataRecoveryToolStyle : public FSlateStyleSet
{
public:
	static void Register();
	static void Unregister();
	static void Shutdown();

	/** @return The Slate style set for data recovery tool widgets */
	static const FDataRecoveryToolStyle& Get();

	static void ReinitializeStyle();
	void InitPadding();

private:
	FDataRecoveryToolStyle();

	static TSharedPtr<FDataRecoveryToolStyle> DataRecoveryToolStyle;
};
