// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRecoveryToolUtils.h"
#include "UObject/PropertyBagRepository.h"

#define LOCTEXT_NAMESPACE "DataRecoveryToolModule"


TObjectPtr<UObject> UE::DataRecoveryTool::Utils::GetInstanceDataObjectFromSelectedClassPath(const FTopLevelAssetPath& ClassPath,
	TObjectPtr<UObject>* OutOwner)
{
	const UClass* LoadedClass = Cast<UClass>(StaticLoadObject(UClass::StaticClass(), nullptr, ClassPath.ToString()));

	const UObject* FoundInstance = FPropertyBagRepository::Get().FindFirstInstanceThatNeedsFixup(LoadedClass);

	if (OutOwner)
	{
		*OutOwner = const_cast<UObject*>(FoundInstance);
	}

	return FoundInstance ? FPropertyBagRepository::Get().FindInstanceDataObject(FoundInstance) : nullptr;
}

void UE::DataRecoveryTool::Utils::ApplyTransforms(const TWeakPtr<TMap<FTopLevelAssetPath, FInstanceDataTransformSet>>& StagedTransformsWeak,
	UObject*& IDO, TNotNull<UObject*> Owner)
{
	const TSharedPtr StagedTransforms = StagedTransformsWeak.Pin();

	if (StagedTransforms == nullptr)
	{
		return;
	}

	for (const TPair<FTopLevelAssetPath, FInstanceDataTransformSet>& TransformPair : *StagedTransforms)
	{
		FInstanceDataTransformSet TransformSet = TransformPair.Value;
		IDO = FInstanceDataTransforms::Get().ApplyTransformSet(TransformPair.Value, IDO, Owner);
	}
}

bool UE::DataRecoveryTool::Utils::ContainsPath(const FTopLevelAssetPath& Parent, const FTopLevelAssetPath& Candidate)
{
	const FString ParentPackage = Parent.GetPackageName().ToString();
	const FString CandidatePackage = Candidate.GetPackageName().ToString();

	return CandidatePackage.StartsWith(ParentPackage, ESearchCase::IgnoreCase);
}

bool UE::DataRecoveryTool::Utils::Snapshot::IsASnapshot(const UObject* Instance)
{
	return Instance->HasAllFlags(FlagsToAdd);
}

void UE::DataRecoveryTool::Utils::UpdateTedsIdoAlertColumn(
	Editor::DataStorage::ICoreProvider* Storage,
	Editor::DataStorage::RowHandle Row,
	const uint32 InstanceWithIdoCount
)
{
	using namespace Editor::DataStorage;
	const FText AlertMessagePattern = LOCTEXT("UnknownPropertiesAlertMessage",
		"Unknown properties found in {InstanceCount} {InstanceCount}|plural(one=instance, other=instances).");

	FFormatNamedArguments AlertMessageArgs;
	AlertMessageArgs.Add(TEXT("InstanceCount"), InstanceWithIdoCount);

	const FText AlertMessage = FText::Format(AlertMessagePattern, AlertMessageArgs);

	static FName AlertName("UnknownPropertiesAlert");

	Alerts::AddAlert(*Storage, Row, AlertName, AlertMessage, Columns::FAlertColumnType::Warning);
}

#undef LOCTEXT_NAMESPACE