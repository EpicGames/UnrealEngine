// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/InstanceDataTransforms.h"
#include "TedsAlertColumns.h"
#include "TedsAlerts.h"


namespace UE::DataRecoveryTool::Utils
{
	TObjectPtr<UObject> GetInstanceDataObjectFromSelectedClassPath(const FTopLevelAssetPath& ClassPath, TObjectPtr<UObject>* OutOwner = nullptr);

	void ApplyTransforms(const TWeakPtr<TMap<FTopLevelAssetPath, FInstanceDataTransformSet>>& StagedTransformsWeak, UObject*& IDO, TNotNull<UObject*> Owner);

	bool ContainsPath(const FTopLevelAssetPath& Parent, const FTopLevelAssetPath& Candidate);

	namespace Snapshot
	{
		constexpr EObjectFlags FlagsToAdd = RF_Transient | RF_DuplicateTransient;
		constexpr EObjectFlags FlagsToRemove = RF_Public | RF_Standalone;

		bool IsASnapshot(const UObject* Instance);
	}

	void UpdateTedsIdoAlertColumn(
		Editor::DataStorage::ICoreProvider* Storage,
		Editor::DataStorage::RowHandle Row,
		const uint32 InstanceWithIdoCount
	);
}

