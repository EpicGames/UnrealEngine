// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstanceDataObjectFixupDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstanceDataObjectFixupPanel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Serialization/ObjectReader.h"
#include "UObject/PropertyBagRepository.h"


static bool bEnableTypeConversions = true;

static FAutoConsoleVariableRef CVarEnableTypeConversions(
	TEXT("IDO.DataRecoveryTool.TypeConversions"),
	bEnableTypeConversions,
	TEXT("Enable the menu entry to convert types in the data recovery tool"));

#define LOCTEXT_NAMESPACE "InstanceDataObjectFixupDetails"

/////////////////////////////////////////////////////////////////
// FInstanceDataObjectNameWidgetOverride
/////////////////////////////////////////////////////////////////

FInstanceDataObjectNameWidgetOverride::FInstanceDataObjectNameWidgetOverride(const TSharedRef<FInstanceDataObjectFixupPanel>& InDiffPanel)
	: DiffPanel(InDiffPanel)
{
}

TSharedRef<SWidget> FInstanceDataObjectNameWidgetOverride::CustomizeName(TSharedRef<SWidget> InnerNameContent, FPropertyPath& Path)
{
	const TSharedRef<SWidget> NameContent = SNew(SWidgetSwitcher)
		.WidgetIndex(this, &FInstanceDataObjectNameWidgetOverride::GetNameWidgetIndex, Path)
		+SWidgetSwitcher::Slot()
		[
			InnerNameContent
		]
		+SWidgetSwitcher::Slot()
		[
			SNew(SComboButton)
			.Visibility(EVisibility::Visible)
			.OnGetMenuContent_Raw(this, &FInstanceDataObjectNameWidgetOverride::GeneratePropertyRedirectMenu, Path)
			.ButtonContent()
			[
				InnerNameContent
			]
		];
	return NameContent;
}


TSet<FPropertyPath> FInstanceDataObjectNameWidgetOverride::GetRedirectOptions(const UStruct* Struct, void* Value) const
{
	TSet<FPropertyPath> Result;
	GetRedirectOptions(Struct, Value, {}, Result);
	return Result;
}

void FInstanceDataObjectNameWidgetOverride::GetRedirectOptions(const UStruct* Struct, void* Value, const FPropertyPath& Path, TSet<FPropertyPath>& OutPaths) const
{
	for (FProperty* SubProperty : TFieldRange<FProperty>(Struct))
	{
		if (SubProperty->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}
		if (!SubProperty->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst))
		{
			continue;
		}
		if (SubProperty->ArrayDim == 1)
		{
			const FPropertyPath SubPath = Path.ExtendPath(FPropertyInfo(SubProperty)).Get();
			GetRedirectOptions(SubProperty, SubProperty->ContainerPtrToValuePtr<void>(Value), SubPath, OutPaths);
		}
		else
		{
			for (int32 ArrayIndex = 0; ArrayIndex < SubProperty->ArrayDim; ++ArrayIndex)
			{
    			const FPropertyPath SubPath = Path.ExtendPath(FPropertyInfo(SubProperty, ArrayIndex)).Get();
    			GetRedirectOptions(SubProperty, SubProperty->ContainerPtrToValuePtr<void>(Value, ArrayIndex), SubPath, OutPaths);
			}
		}
    }
}

void FInstanceDataObjectNameWidgetOverride::GetRedirectOptions(const FProperty* Property, void* Value, const FPropertyPath& Path, TSet<FPropertyPath>& OutPaths) const
{
	if (Property->GetBoolMetaData(TEXT("isLoose")))
	{
		// don't include loose properties as options
		return;
	}

	if (!DiffPanel.IsValid())
	{
		return;
	}

	if (!DiffPanel.Pin()->PropertyNeedsRedirect(Path))
	{
		OutPaths.Add(Path);
	}
	
	if (const FStructProperty* AsStructProperty = CastField<FStructProperty>(Property))
	{
		GetRedirectOptions(AsStructProperty->Struct, Value, Path, OutPaths);
	}
	if (const FObjectProperty* AsObjectProperty = CastField<FObjectProperty>(Property))
	{
		if (AsObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
		{
			if (TObjectPtr<UObject> Object = AsObjectProperty->GetObjectPropertyValue(Value))
			{
				UE::FPropertyBagRepository& PropertyBagRepository = UE::FPropertyBagRepository::Get();
				if (UObject* Found = PropertyBagRepository.FindInstanceDataObject(Object))
				{
					Object = Found;
				}
				GetRedirectOptions(Object->GetClass(), Object, Path, OutPaths);
			}
		}
	}
	else if (const FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Property))
	{
		FScriptArrayHelper Array(AsArrayProperty, Value);
		for (int32 ArrayIndex = 0; ArrayIndex < Array.Num(); ++ArrayIndex)
		{
			const FPropertyPath SubPath = Path.ExtendPath(FPropertyInfo(AsArrayProperty->Inner, ArrayIndex)).Get();
			GetRedirectOptions(AsArrayProperty->Inner, Array.GetElementPtr(ArrayIndex), SubPath, OutPaths);
		}
	}
	else if (const FSetProperty* AsSetProperty = CastField<FSetProperty>(Property))
	{
		FScriptSetHelper Set(AsSetProperty, Value);
		for (FScriptSetHelper::FIterator Iterator = Set.CreateIterator(); Iterator; ++Iterator)
		{
			const FPropertyPath SubPath = Path.ExtendPath(FPropertyInfo(AsSetProperty->ElementProp, Iterator.GetLogicalIndex())).Get();
			GetRedirectOptions(AsSetProperty->ElementProp, Set.GetElementPtr(Iterator), SubPath, OutPaths);
		}
	}
	else if (const FMapProperty* AsMapProperty = CastField<FMapProperty>(Property))
	{
		FScriptMapHelper Map(AsMapProperty, Value);
		for (FScriptMapHelper::FIterator Iterator = Map.CreateIterator(); Iterator; ++Iterator)
		{
			const FPropertyPath SubPath = Path.ExtendPath(FPropertyInfo(AsMapProperty->ValueProp, Iterator.GetLogicalIndex())).Get();
			GetRedirectOptions(AsMapProperty->ValueProp, Map.GetValuePtr(Iterator), SubPath, OutPaths);
		}
	}
}

int32 FInstanceDataObjectNameWidgetOverride::GetNameWidgetIndex(FPropertyPath Path) const
{
	if (const TSharedPtr<FInstanceDataObjectFixupPanel> Panel = DiffPanel.Pin())
	{
		if (Panel->HasViewFlag(FInstanceDataObjectFixupPanel::EViewFlags::AllowRemapLooseProperties))
		{
			if (Panel->PropertyNeedsRedirect(Path))
			{
				return DisplayRedirectMenu;
			}
		}
	}
	return DisplayRegularName;
}

static FText GetCleanVersePath(const FPropertyPath& Path)
{
	FString PathString = Path.ToString();
	int32 I = PathString.Find(TEXT("__verse_0x"));
	while (I < GetNum(PathString) && I != INDEX_NONE)
	{
		PathString.RemoveAt(I, 19, EAllowShrinking::No);
		I = PathString.Find(TEXT("__verse_0x"), ESearchCase::IgnoreCase, ESearchDir::FromStart, I);
	}
	return FText::FromString(PathString);
}

static bool DoesRedirectRespectContainerIndices(const FPropertyPath& Src, const FPropertyPath& Dst)
{
	for (int Index = 0; Index < Src.GetNumProperties() && Index < Dst.GetNumProperties(); ++Index)
	{
		const FPropertyInfo& SrcInfo = Src.GetPropertyInfo(Index);
		const FPropertyInfo& DstInfo = Dst.GetPropertyInfo(Index);
		if (SrcInfo.ArrayIndex != DstInfo.ArrayIndex)
		{
			return false;
		}
	}
	return true;
}

static bool PropertiesAreMatchingTypes(const FProperty* A, const FProperty* B)
{
	return UE::FPropertyTypeName(A) == UE::FPropertyTypeName(B);
}

TSharedRef<SWidget> FInstanceDataObjectNameWidgetOverride::GeneratePropertyRedirectMenu(FPropertyPath Path) const
{
	FMenuBuilder MenuBuilder(true, nullptr);
	
	const TSharedPtr<FInstanceDataObjectFixupPanel> Panel = DiffPanel.Pin();
	if (!Panel)
	{
		return MenuBuilder.MakeWidget();
	}
	
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("Delete", "Delete"));
	{
		FText DisplayName = LOCTEXT("MarkForDeletion", "Mark For Deletion");
		FText Tooltip = LOCTEXT("MarkForDeletionTooltip", "Mark this property for deletion");
		MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), FName("Icons.Delete"))
					, FUIAction(FExecuteAction::CreateSP(Panel.Get(), &FInstanceDataObjectFixupPanel::OnMarkForDelete, Path))
					, NAME_None);
	}
	MenuBuilder.EndSection();

	TSet<FPropertyPath> RedirectOptions;
	if (Panel->InstanceDataObject)
	{
		UObject* FirstInstanceDataObject = Panel->InstanceDataObject;
		RedirectOptions = GetRedirectOptions(FirstInstanceDataObject->GetClass(), FirstInstanceDataObject);
	}

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("MoveProperty", "Move"));
	{
		for (const FPropertyPath& Option : RedirectOptions)
		{
			if (!DoesRedirectRespectContainerIndices(Path, Option))
			{
				continue;
			}
			const FProperty* ThisProperty = Path.GetLeafMostProperty().Property.Get();
			const FProperty* OptionProperty = Option.GetLeafMostProperty().Property.Get();
			if (ThisProperty->GetFName() == OptionProperty->GetFName())
			{
				if (PropertiesAreMatchingTypes(OptionProperty, ThisProperty))
				{
					FText DisplayName = GetCleanVersePath(Option);
					FText Tooltip = FText::Format(LOCTEXT("MovePropertyTooltip", "Move property to '{0}'"), DisplayName);
					MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), FName("Icons.Convert"))
					, FUIAction(FExecuteAction::CreateSP(Panel.Get(), &FInstanceDataObjectFixupPanel::OnRedirectProperty, Path, Option))
					, NAME_None);
				}
			}
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RenameProperty", "Rename"));
	{
		for (const FPropertyPath& Option : RedirectOptions)
		{
			if (!DoesRedirectRespectContainerIndices(Path, Option))
			{
				continue;
			}
			const FProperty* ThisProperty = Path.GetLeafMostProperty().Property.Get();
			const FProperty* OptionProperty = Option.GetLeafMostProperty().Property.Get();
			if (ThisProperty->GetFName() == OptionProperty->GetFName())
			{
				continue; // handled in the "move" category
			}
			if (PropertiesAreMatchingTypes(OptionProperty, ThisProperty))
			{
				FText DisplayName = GetCleanVersePath(Option);
				FText Tooltip = FText::Format(LOCTEXT("RenamePropertyTooltip", "Rename property to '{0}'"), DisplayName);
				MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), FName("Icons.Convert"))
				, FUIAction(FExecuteAction::CreateSP(Panel.Get(), &FInstanceDataObjectFixupPanel::OnRedirectProperty, Path, Option))
				, NAME_None);
			}
		}
	}
	MenuBuilder.EndSection();

	if (bEnableTypeConversions)
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("ChangeToDifferingTypeConversion", "Convert Type"));
		{
			for (const FPropertyPath& Option : RedirectOptions)
			{
				const FProperty* ThisProperty = Path.GetLeafMostProperty().Property.Get();
				const FProperty* OptionProperty = Option.GetLeafMostProperty().Property.Get();
				if (PropertiesAreMatchingTypes(OptionProperty, ThisProperty))
				{
					continue; // same type handled above
				}
				if (ThisProperty->GetFName() != OptionProperty->GetFName())
				{
					continue; // renames to handled below
				}
				if (Panel->CanConvert(Path, const_cast<FProperty*>(OptionProperty)))
				{
					FText DisplayName = GetCleanVersePath(Option);
					FText Tooltip = FText::Format(LOCTEXT("ConvertPropertyTooltip", "Convert property to '{0}'"), DisplayName);
					MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), FName("Icons.Convert"))
						, FUIAction(FExecuteAction::CreateSP(Panel.Get(), &FInstanceDataObjectFixupPanel::OnRedirectProperty, Path, Option))
						, NAME_None);
				}
			}
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection(NAME_None, LOCTEXT("RenameToDifferingTypeConversion", "Convert Type and Rename"));
		{
			for (const FPropertyPath& Option : RedirectOptions)
			{
				const FProperty* ThisProperty = Path.GetLeafMostProperty().Property.Get();
				const FProperty* OptionProperty = Option.GetLeafMostProperty().Property.Get();
				if (PropertiesAreMatchingTypes(OptionProperty, ThisProperty))
				{
					continue; // same type handled above
				}
				if (ThisProperty->GetFName() == OptionProperty->GetFName())
				{
					continue; // renames to handled below
				}
				if (Panel->CanConvert(Path, const_cast<FProperty*>(OptionProperty)))
				{
					FText DisplayName = GetCleanVersePath(Option);
					FText Tooltip = FText::Format(LOCTEXT("ConvertPropertyTooltip", "Convert property to '{0}'"), DisplayName);
					MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), FName("Icons.Convert"))
						, FUIAction(FExecuteAction::CreateSP(Panel.Get(), &FInstanceDataObjectFixupPanel::OnRedirectProperty, Path, Option))
						, NAME_None);
				}
			}
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
