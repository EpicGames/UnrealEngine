// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <DetailsNameWidgetOverrideCustomization.h>

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "Containers/Set.h"
#include "PropertyPath.h"
#include "Widgets/SWidget.h"

#define UE_API INSTANCEDATAOBJECTFIXUPTOOL_API

class IDetailCategoryBuilder;
class FInstanceDataObjectFixupPanel;

class FInstanceDataObjectNameWidgetOverride : public FDetailsNameWidgetOverrideCustomization
{
public:
	UE_API FInstanceDataObjectNameWidgetOverride(const TSharedRef<FInstanceDataObjectFixupPanel>& DiffPanel);
	virtual ~FInstanceDataObjectNameWidgetOverride() override = default;
	UE_API virtual TSharedRef<SWidget> CustomizeName(TSharedRef<SWidget> InnerNameContent, FPropertyPath& Path) override;
private:
	
	UE_API int32 GetNameWidgetIndex(FPropertyPath Path) const;
	UE_API TSharedRef<SWidget> GeneratePropertyRedirectMenu(FPropertyPath Path) const;
	
	UE_API TSet<FPropertyPath> GetRedirectOptions(const UStruct* Struct, void* Value) const;
	UE_API void GetRedirectOptions(const UStruct* Struct, void* Value, const FPropertyPath& Path, TSet<FPropertyPath>& OutPaths) const;
	UE_API void GetRedirectOptions(const FProperty* Property, void* Value, const FPropertyPath& Path, TSet<FPropertyPath>& OutPaths) const;
	
	TWeakPtr<FInstanceDataObjectFixupPanel> DiffPanel;

	enum ENameWidgetIndex : uint8
	{
		DisplayRegularName = 0,
		DisplayRedirectMenu = 1
	};
};

#undef UE_API
