// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstanceDataObjectFixupPanel.h"

#include "AsyncDetailViewDiff.h"
#include "DetailTreeNode.h"
#include "Widgets/Layout/LinkableScrollBar.h"
#include "InstanceDataObjectFixupDetailCustomization.h"
#include "Misc/StringOutputDevice.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/PropertyStateTracking.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Serialization/ObjectReader.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

#include "UObject/OverriddenPropertySet.h"
#include "UObject/OverridableManager.h"

#include "UObject/PropertyPathNameTree.h"
#include "UObject/InstanceDataTransforms.h"
#include "UObject/UObjectThreadContext.h"

#define LOCTEXT_NAMESPACE "InstanceDataObjectFixupPanel"

static const FName NAME_IsLooseMetadata(TEXT("IsLoose"));

static void* ResolvePath(const FPropertyPath& Path, void* Value)
{
	for (int32 PathIndex = 0; PathIndex < Path.GetNumProperties(); ++PathIndex)
	{
		const FPropertyInfo& PropertyInfo = Path.GetPropertyInfo(PathIndex);
		const FProperty* Property = PropertyInfo.Property.Get();
		if (!Property)
		{
			return nullptr;
		}

		Value = Property->ContainerPtrToValuePtr<void>(Value, PropertyInfo.ArrayIndex != INDEX_NONE ? PropertyInfo.ArrayIndex : 0);

		if (const FObjectProperty* AsObjectProperty = CastField<FObjectProperty>(Property))
		{
			UObject* Object = AsObjectProperty->GetObjectPropertyValue(Value);
			UE::FPropertyBagRepository& PropertyBagRepository = UE::FPropertyBagRepository::Get();
			if (UObject* Found = PropertyBagRepository.FindInstanceDataObject(Object))
			{
				Object = Found;
			}
			Value = Object;
		}
		else if (PathIndex + 1 < Path.GetNumProperties())
		{
			if (const FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Property))
			{
				FScriptArrayHelper Helper(AsArrayProperty, Value);
				Value = Helper.GetElementPtr(Path.GetPropertyInfo(++PathIndex).ArrayIndex);
			}
			if (const FSetProperty* AsSetProperty = CastField<FSetProperty>(Property))
			{
				FScriptSetHelper Helper(AsSetProperty, Value);
				Value = Helper.FindNthElementPtr(Path.GetPropertyInfo(++PathIndex).ArrayIndex);
			}
			if (const FMapProperty* AsMapProperty = CastField<FMapProperty>(Property))
			{
				FScriptMapHelper Helper(AsMapProperty, Value);
				Value = Helper.FindNthValuePtr(Path.GetPropertyInfo(++PathIndex).ArrayIndex);
			}
		}
	}

	return Value;
}


bool FInstanceDataObjectFixupPanel::CanConvert(const FPropertyPath& SourcePropertyPath, FProperty* DestinationProperty)
{
	FProperty* SourceProperty = SourcePropertyPath.GetLeafMostProperty().Property.Get();

	if (SourceProperty->GetID() == DestinationProperty->GetID())
	{
		return true;
	}

	const void* SourceData = ResolvePath(SourcePropertyPath, InstanceDataObject);

	if (SourceData == nullptr)
	{
		return false;
	}

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	TGuardValue<bool> ScopedImpersonateProperties(SerializeContext->bImpersonateProperties, true);

	TArray<uint8, TInlineAllocator<64>> Buffer;
	TMemoryWriterBase<TInlineAllocator<64>> MemoryWriter(Buffer);
	FStructuredArchiveFromArchive StructuredWriter(MemoryWriter);

	SourceProperty->SerializeItem(StructuredWriter.GetSlot(), (uint8*)SourceData);
	FMemoryReaderView MemoryReader(Buffer);
	FStructuredArchiveFromArchive StructuredReader(MemoryReader);


	TArray<uint8, TInlineAllocator<64>> SourceToDest;
	SourceToDest.SetNumUninitialized(DestinationProperty->GetElementSize());
	DestinationProperty->InitializeValue(SourceToDest.GetData());
	void* DestinationContainer = SourceToDest.GetData();

	// todo: handle static arrays
	FPropertyTag SourceTag(SourceProperty, 0, (uint8*)SourceData);

	bool bResult = false;
	switch (DestinationProperty->ConvertFromType(SourceTag, StructuredReader.GetSlot(), (uint8*)DestinationContainer, SourceProperty->GetOwnerStruct(), nullptr))
	{
	case EConvertFromTypeResult::UseSerializeItem:
		break;
	case EConvertFromTypeResult::Serialized:
		bResult = true;
		break;
	case EConvertFromTypeResult::CannotConvert:
		break;
	case EConvertFromTypeResult::Converted:
		bResult = true;
		break;
	}

	DestinationProperty->DestroyValue(SourceToDest.GetData());

	return bResult;
}

static bool IsPathOverridden(UObject* Object, const FPropertyPath& Path)
{
	FPropertyVisitorPath VisitorPath;
	for (int32 I = 0; I < Path.GetNumProperties(); ++I)
	{
		FProperty* Property = Path.GetPropertyInfo(I).Property.Get();
		int32 Index = Path.GetPropertyInfo(I).ArrayIndex;
		EPropertyVisitorInfoType SegmentType = (Index != INDEX_NONE) ? EPropertyVisitorInfoType::StaticArrayIndex : EPropertyVisitorInfoType::None;
		if (I > 0 && Index != INDEX_NONE)
		{
			FProperty* ParentProperty = Path.GetPropertyInfo(I - 1).Property.Get();
			if (ParentProperty->IsA<FArrayProperty>() || ParentProperty->IsA<FSetProperty>())
			{
				SegmentType = EPropertyVisitorInfoType::ContainerIndex;
			}
			else if (FMapProperty* MapProperty = CastField<FMapProperty>(ParentProperty))
			{
				if (Property == MapProperty->KeyProp)
				{
					SegmentType = EPropertyVisitorInfoType::MapKey;
				}
				else if (Property == MapProperty->ValueProp)
				{
					SegmentType = EPropertyVisitorInfoType::MapValue;
				}
			}
		}
		VisitorPath.Push(FPropertyVisitorInfo(Property, Index, SegmentType));
	}
	EOverriddenPropertyOperation Override = FOverridableManager::Get().GetOverriddenPropertyOperation(Object, VisitorPath);
	return (Override != EOverriddenPropertyOperation::None);
}
FInstanceDataObjectFixupPanel::FInstanceDataObjectFixupPanel(
	const TObjectPtr<UObject>& InstanceDataObject,
	EViewFlags ViewFlags, 
	const TWeakPtr<TMap<FTopLevelAssetPath,
	UE::FInstanceDataTransformSet>>& StagedTransforms
)
	: InstanceDataObject(InstanceDataObject)
	, ViewFlags(ViewFlags)
	, StagedTransforms(StagedTransforms)
{
}

static bool RemoveCustomizationsWithLooseProperties(const FFieldVariant& FieldVariant, const TSharedPtr<IDetailsView>& DetailsView)
{
#if WITH_EDITORONLY_DATA
	if (FStructProperty* AsStructProperty = FieldVariant.Get<FStructProperty>())
	{
		if (RemoveCustomizationsWithLooseProperties(AsStructProperty->Struct, DetailsView))
		{
			return true;
		}
	}
	else if (FObjectProperty* AsObjectProperty = FieldVariant.Get<FObjectProperty>())
	{
		if (AsObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
		{
			if (RemoveCustomizationsWithLooseProperties(AsObjectProperty->PropertyClass, DetailsView))
			{
				return true;
			}
		}
	}
	else if (const FArrayProperty* AsArrayProperty = FieldVariant.Get<FArrayProperty>())
	{
		if (RemoveCustomizationsWithLooseProperties(AsArrayProperty->Inner, DetailsView))
		{
			return true;
		}
	}
	else if (const FSetProperty* AsSetProperty = FieldVariant.Get<FSetProperty>())
	{
		if (RemoveCustomizationsWithLooseProperties(AsSetProperty->ElementProp, DetailsView))
		{
			return true;
		}
	}
	else if (const FMapProperty* AsMapProperty = FieldVariant.Get<FMapProperty>())
	{
		if (RemoveCustomizationsWithLooseProperties(AsMapProperty->KeyProp, DetailsView))
		{
			return true;
		}
		if (RemoveCustomizationsWithLooseProperties(AsMapProperty->ValueProp, DetailsView))
		{
			return true;
		}
	}
	else if (UStruct* AsStruct = FieldVariant.Get<UStruct>())
	{
		bool result = false;
		for (const FProperty* Property : TFieldRange<FProperty>(AsStruct))
		{
			if (RemoveCustomizationsWithLooseProperties(Property, DetailsView))
			{
				result = true;
			}
		}
		if (result)
		{
			// register an empty delegate to override the global rule of displaying this type with customizations
			DetailsView->RegisterInstancedCustomPropertyTypeLayout(AsStruct->GetFName(), {});
		}
		return result;
	}
	
	if (const FProperty* Property = FieldVariant.Get<FProperty>())
	{
		if (Property->HasMetaData(NAME_IsLooseMetadata))
		{
			return true;
		}
	}
#endif
	return false;
}

void FInstanceDataObjectFixupPanel::RefreshDetailsView()
{
	DetailsView->SetObject(InstanceDataObject, true);
}

TSharedPtr<IDetailsView>& FInstanceDataObjectFixupPanel::GenerateDetailsView(bool bScrollbarOnLeft)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.ExternalScrollbar = SAssignNew(LinkableScrollBar, SLinkableScrollBar);
	DetailsViewArgs.ScrollbarAlignment = bScrollbarOnLeft ? HAlign_Left : HAlign_Right;
	DetailsViewArgs.DetailsNameWidgetOverrideCustomization = MakeShared<FInstanceDataObjectNameWidgetOverride>(SharedThis(this));
	DetailsViewArgs.bResolveInstanceDataObjects.Emplace(true);
	DetailsViewArgs.bShowLooseProperties = !HasViewFlag(EViewFlags::HideLooseProperties);
	DetailsViewArgs.bLockable = false;                                               
	DetailsViewArgs.bAllowSearch = false;                                            
	DetailsViewArgs.bSearchInitialKeyFocus = false;                                  
	DetailsViewArgs.bShowOptions = false;                                            
	DetailsViewArgs.bAllowFavoriteSystem = false;                                    
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	if (HasViewFlag(EViewFlags::IncludeOnlySetBySerialization))
	{
		DetailsViewArgs.ShouldForceHideProperty.BindLambda([this](const TSharedRef<IPropertyHandle>& PropertyNode)->bool
		{
			TSharedPtr<FPropertyPath> PropertyPath = PropertyNode->CreateFPropertyPath();
			if (PropertyPath)
			{
				return !IsPathOverridden(InstanceDataObject, *PropertyPath);
			}
			return false;
		});
	}
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	RemoveCustomizationsWithLooseProperties(InstanceDataObject->GetClass(), DetailsView);

	RefreshDetailsView();

	FCoreUObjectDelegates::OnObjectsReinstanced.AddSPLambda(this,
		[this](const TMap<UObject*, UObject*>& ObjectMap)
		{
			if (UObject* const* Found = ObjectMap.Find(InstanceDataObject))
			{
				InstanceDataObject = TObjectPtr<UObject>(*Found);
				DetailsView->SetObject(InstanceDataObject, true);
			}
		});
	return DetailsView;
}

void FInstanceDataObjectFixupPanel::SetDiffAgainstLeft(const TSharedPtr<FAsyncDetailViewDiff>& InDiffAgainstLeft)
{
	DiffAgainstLeft = InDiffAgainstLeft;
}

void FInstanceDataObjectFixupPanel::SetDiffAgainstRight(const TSharedPtr<FAsyncDetailViewDiff>& InDiffAgainstRight)
{
	DiffAgainstRight = InDiffAgainstRight;
}

TSharedPtr<FAsyncDetailViewDiff> FInstanceDataObjectFixupPanel::GetDiffAgainstLeft() const
{
	return DiffAgainstLeft.Pin();
}

TSharedPtr<FAsyncDetailViewDiff> FInstanceDataObjectFixupPanel::GetDiffAgainstRight() const
{
	return DiffAgainstRight.Pin();
}

bool FInstanceDataObjectFixupPanel::AreAllConflictsRedirected() const
{
	UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();

	const UObject* Owner = Repository.FindInstanceForDataObject(InstanceDataObject);
	return Owner == nullptr || !Repository.RequiresFixup(Owner);
}

void FInstanceDataObjectFixupPanel::AutoApplyMarkDeletedActions()
{
	const TSharedPtr<FAsyncDetailViewDiff> Diff = DiffAgainstRight.Pin();
	if (!Diff)
	{
		return;
	}

	Diff->ForEach(ETreeTraverseOrder::PreOrder,
		[this] (const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)->ETreeTraverseControl
		{
			if (DiffNode->DiffResult == ETreeDiffResult::MissingFromTree2)
			{
				if (const TSharedPtr<FDetailTreeNode> LeftTreeNode = DiffNode->ValueA.Pin())
				{
					const FPropertyPath Path = LeftTreeNode->GetPropertyPath();
					if (Path.IsValid())
					{
						MarkForDelete(Path);

						return ETreeTraverseControl::SkipChildren;
					}
				}
			}
			
			return ETreeTraverseControl::Continue;
		});
}

bool FInstanceDataObjectFixupPanel::HasViewFlag(EViewFlags Flag)
{
	return static_cast<uint8>(Flag) & static_cast<uint8>(ViewFlags);
}

static UStruct* ExtractStructFromType(UE::FPropertyTypeName Type, const UE::FPropertyPathName& Path, int32 SegmentIndex)
{
	static const FName NAME_Key(ANSITEXTVIEW("Key"));
	static const FName NAME_Value(ANSITEXTVIEW("Value"));
	if (Type.GetName() == NAME_StructProperty)
	{
		UE::FPropertyTypeName StructType = Type.GetParameter(0);
		FName StructName = StructType.GetName();
		FName PackageName = StructType.GetParameter(0).GetName();
		FTopLevelAssetPath StructAssetPath(PackageName, StructName);
		if (UScriptStruct* Struct = Cast<UScriptStruct>(StaticFindObject(UStruct::StaticClass(), StructAssetPath)))
		{
			return Struct;
		}
	}
	else if (Type.GetName() == NAME_ArrayProperty || Type.GetName() == NAME_SetProperty)
	{
		UE::FPropertyTypeName InnerType = Type.GetParameter(0);
		return ExtractStructFromType(InnerType, Path, SegmentIndex);
	}
	else if (Type.GetName() == NAME_MapProperty)
	{
		if (SegmentIndex + 1 < Path.GetSegmentCount())
		{
			UE::FPropertyPathNameSegment MapInternalSegment = Path.GetSegment(SegmentIndex + 1);
			if (MapInternalSegment.Name == NAME_Key)
			{
				UE::FPropertyTypeName KeyType = Type.GetParameter(0);
				return ExtractStructFromType(KeyType, Path, SegmentIndex);
			}
			else if (MapInternalSegment.Name == NAME_Value)
			{
				UE::FPropertyTypeName ValType = Type.GetParameter(1);
				return ExtractStructFromType(ValType.GetParameter(0), Path, SegmentIndex);
			}
		}

	}
	return nullptr;
}

static int32 GetLastSharedStructSegment(const UE::FPropertyPathName& PathA, const UE::FPropertyPathName& PathB, UStruct*& OutStruct)
{
	int32 LastStructIndex = INDEX_NONE;
	const int32 Size = FMath::Min(PathA.GetSegmentCount(), PathB.GetSegmentCount());
	for (int32 SegmentIndex = 0; SegmentIndex < Size; ++SegmentIndex)
	{
		UE::FPropertyPathNameSegment SegmentA = PathA.GetSegment(SegmentIndex);
		UE::FPropertyPathNameSegment SegmentB = PathB.GetSegment(SegmentIndex);
		if (SegmentA.Index != SegmentB.Index || SegmentA.Name != SegmentB.Name || SegmentA.Type != SegmentB.Type)
		{
			break;
		}

		if (UStruct* Found = ExtractStructFromType(SegmentA.Type, PathA, SegmentIndex))
		{
			OutStruct = Found;
			LastStructIndex = SegmentIndex;
		}
	}
	return LastStructIndex;
}

UE::FInstanceDataTransformSet* FInstanceDataObjectFixupPanel::GetStagedTransformsForStruct(UStruct* ClassOrStruct)
{
	check(ClassOrStruct);
	if (TSharedPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>> Transforms = StagedTransforms.Pin())
	{
		FTopLevelAssetPath Key = ClassOrStruct->GetStructPathName();
		if (UE::FInstanceDataTransformSet* Found = Transforms->Find(Key))
		{
			return Found;
		}
		return &Transforms->Add(Key, UE::FInstanceDataTransforms::Get().GetTransformSet(*ClassOrStruct));
	}
	return nullptr;
}

void FInstanceDataObjectFixupPanel::RedirectProperty(const FPropertyPath& From, const FPropertyPath& To)
{
	TArray<UE::FPropertyPathName> FromPathNames = From.ToPropertyPathName();
	TArray<UE::FPropertyPathName> ToPathNames = To.ToPropertyPathName();

	if (FromPathNames.Num() != 1 || ToPathNames.Num() != 1)
	{
		ensureMsgf(false, TEXT("Redirecting into or out of subobjects is not supported yet!"));
		return;
	}

	// So that this IDT operation can be used on as many struct instances as possible, find the inner-most struct that's shared by both paths and chop them to be relative
	// to that struct. So for example 'From' is "Foo (MyFooStruct) -> Bar (MyBarStruct) -> X (Float)" and 'To' is "Foo (MyFooStruct) -> Bar (MyBarStruct) -> Y (Float)"
	// then we'll create an IDT for ALL instances of MyBarStruct that remaps "X (Float)" to "Y (Float)" instead of only remapping the full path.
	const UObject* Owner = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(InstanceDataObject);
	UStruct* BaseStruct = Owner->GetClass();
	int32 StructIndex = GetLastSharedStructSegment(FromPathNames[0], ToPathNames[0], BaseStruct);

	UE::FPropertyPathName SrcPath;
	UE::FPropertyPathName DstPath;
	for (int32 SegmentIndex = StructIndex + 1; SegmentIndex < FromPathNames[0].GetSegmentCount() || SegmentIndex < ToPathNames[0].GetSegmentCount(); ++SegmentIndex)
	{
		if (SegmentIndex < ToPathNames[0].GetSegmentCount())
		{
			DstPath.Push(ToPathNames[0].GetSegment(SegmentIndex));
		}
		if (SegmentIndex < FromPathNames[0].GetSegmentCount())
		{
			SrcPath.Push(FromPathNames[0].GetSegment(SegmentIndex));
		}
	}
	
	if (UE::FInstanceDataTransformSet* Transforms = GetStagedTransformsForStruct(BaseStruct))
	{
		TSharedPtr<FJsonObject> Params = MakeShared<FJsonObject>();
		Params->SetField(TEXT("src"), SrcPath.ToJsonValue());
		Params->SetField(TEXT("dst"), DstPath.ToJsonValue());
		Transforms->Operations.Add({ .OpCode = TEXT("RedirectProperty"), .Params = Params });

		TObjectPtr<UObject> OwnerObjectPtr = const_cast<UObject*>(Owner);
		UE::FInstanceDataTransforms::Get().ApplyTransformSet(*Transforms, InstanceDataObject, OwnerObjectPtr);
	}
}

void FInstanceDataObjectFixupPanel::OnRedirectProperty(FPropertyPath From, FPropertyPath To)
{
	RedirectProperty(From, To);
}

bool FInstanceDataObjectFixupPanel::PropertyNeedsRedirect(const FPropertyPath& Path) const
{
	if (Path.GetLeafMostProperty().Property->HasAnyPropertyFlags(CPF_SkipSerialization))
	{
		return false;
	}

	if (!IsPathOverridden(InstanceDataObject, Path))
	{
		return false;
	}

	TArray<UE::FPropertyPathName> PathNames = Path.ToPropertyPathName();
	if (PathNames.Num() != 1)
	{
		return false;
	}

	const UObject* Owner = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(InstanceDataObject);
	return UE::FPropertyBagRepository::Get().RequiresFixup(Owner, PathNames[0]);
}

void FInstanceDataObjectFixupPanel::MarkForDelete(const FPropertyPath& CurrentPath)
{
	TArray<UE::FPropertyPathName> srcPathNames = CurrentPath.ToPropertyPathName();

	if (srcPathNames.Num() != 1)
	{
		ensureMsgf(false, TEXT("out of subobjects is not supported yet!"));
		return;
	}
	const UObject* Owner = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(InstanceDataObject);

	if (UE::FInstanceDataTransformSet* Transforms = GetStagedTransformsForStruct(Owner->GetClass()))
	{
		TSharedPtr<FJsonObject> Params = MakeShared<FJsonObject>();
		Params->SetField(TEXT("src"), srcPathNames[0].ToJsonValue());
		Transforms->Operations.Add({ .OpCode = TEXT("RemoveProperty"), .Params = Params });

		TObjectPtr<UObject> OwnerObjectPtr = const_cast<UObject*>(Owner);
		UE::FInstanceDataTransforms::Get().ApplyTransformSet(*Transforms, InstanceDataObject, OwnerObjectPtr);
	}
}

void FInstanceDataObjectFixupPanel::OnMarkForDelete(FPropertyPath Path)
{
	MarkForDelete(Path);
}

#undef LOCTEXT_NAMESPACE
