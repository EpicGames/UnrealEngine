// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AsyncDetailViewDiff.h"
#include "UObject/Object.h"
#include "Templates/SharedPointer.h"
#include <UObject/InstanceDataTransforms.h>

class SLinkableScrollBar;

class FInstanceDataObjectFixupPanel : public TSharedFromThis<FInstanceDataObjectFixupPanel>
{
public:
	enum class EViewFlags : uint8
	{
		None = 0,
		HideLooseProperties = (1 << 0), // hide properties with isLoose metadata set to true
		IncludeOnlySetBySerialization = (1 << 1), // hide properties that weren't set by serialization
		ReadonlyValues = (1 << 2),
		AllowRemapLooseProperties = (1 << 3),

		// displays only properties found in the property bag, allow remapping, and disallow value edits
		DefaultLeftPanel = IncludeOnlySetBySerialization | AllowRemapLooseProperties | ReadonlyValues,
		// display only properties found in latest version of the class but allow value edits
		DefaultRightPanel = HideLooseProperties,
	};

	FInstanceDataObjectFixupPanel(const TObjectPtr<UObject>& InstanceDataObject, EViewFlags ViewFlags, const TWeakPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>>& StagedTransforms);

	void RefreshDetailsView();
	TSharedPtr<IDetailsView>& GenerateDetailsView(bool bScrollbarOnLeft = false);
	
	void SetDiffAgainstLeft(const TSharedPtr<FAsyncDetailViewDiff>& DiffAgainstLeft);
	void SetDiffAgainstRight(const TSharedPtr<FAsyncDetailViewDiff>& DiffAgainstRight);
	TSharedPtr<FAsyncDetailViewDiff> GetDiffAgainstLeft() const;
	TSharedPtr<FAsyncDetailViewDiff> GetDiffAgainstRight() const;

	bool AreAllConflictsRedirected() const;

	void MarkForDelete(const FPropertyPath& Path);
	// pass-by-copy version for delegates. Use MarkForDelete when possible
	void OnMarkForDelete(FPropertyPath Path);
	// mark all conflicted properties for delete (FixupMode only)
	void AutoApplyMarkDeletedActions();

	// returns true if the provided path is an unknown property that needs to be redirected via IDT
	bool PropertyNeedsRedirect(const FPropertyPath& Path) const;

	bool HasViewFlag(EViewFlags Flag);

	// Initialized by SInstanceDataObjectFixupTool
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<SLinkableScrollBar> LinkableScrollBar;

private:
	friend class FInstanceDataObjectFixupSpecification; // for access to Redirects
	friend class FInstanceDataObjectNameWidgetOverride;

	UE::FInstanceDataTransformSet* GetStagedTransformsForStruct(UStruct* ClassOrStruct);

	void RedirectProperty(const FPropertyPath& From, const FPropertyPath& To);

	bool CanConvert(const FPropertyPath& SourcePropertyPath, FProperty* DestinationProperty);

	// pass-by-copy version for delegates. Use RedirectProperty when possible
	void OnRedirectProperty(FPropertyPath From, FPropertyPath To);

	TObjectPtr<UObject> InstanceDataObject;

	TWeakPtr<FAsyncDetailViewDiff> DiffAgainstLeft;
	TWeakPtr<FAsyncDetailViewDiff> DiffAgainstRight;
	EViewFlags ViewFlags = EViewFlags::None;
	TWeakPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>> StagedTransforms;
};
