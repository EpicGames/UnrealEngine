// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataObjectFixupTool.h"

#include "DataRecoveryToolStyle.h"
#include "InstanceDataObjectFixupPanel.h"
#include "DetailTreeNode.h"
#include "SlateOptMacros.h"
#include "SDetailsSplitter.h"
#include "Widgets/Layout/LinkableScrollBar.h"
#include "Widgets/Input/SButton.h"
#include "UObject/PropertyBagRepository.h"
#include "DataRecoveryToolUtils.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Engine/Engine.h"

#define USE_SNAPSHOT_INSTANCE 1

#define LOCTEXT_NAMESPACE "InstanceDataObjectFixupTool"

class FInstanceDataObjectFixupSpecification : public TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>
{
public:
	using Super = TTreeDiffSpecification<TWeakPtr<FDetailTreeNode>>;
	
	FInstanceDataObjectFixupSpecification(const TSharedPtr<FInstanceDataObjectFixupPanel>& InLeftPanel, const TSharedPtr<FInstanceDataObjectFixupPanel>& InRightPanel)
		: LeftPanel(InLeftPanel)
		, RightPanel(InRightPanel)
	{}

	// when diffing, use the redirects to match properties so that renames are respected
	virtual bool AreMatching(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const override
	{
		const TSharedPtr<FDetailTreeNode> PinnedTreeNodeA = TreeNodeA.Pin();
		const TSharedPtr<FDetailTreeNode> PinnedTreeNodeB = TreeNodeB.Pin();
		if (!PinnedTreeNodeA || !PinnedTreeNodeB)
		{
			return PinnedTreeNodeA == PinnedTreeNodeB;
		}

		const TSharedPtr<IPropertyHandle> PropertyHandleA = PinnedTreeNodeA->CreatePropertyHandle();
		const TSharedPtr<IPropertyHandle> PropertyHandleB = PinnedTreeNodeB->CreatePropertyHandle();
		if (!PropertyHandleA || !PropertyHandleB)
		{
			// top level category nodes
			return PinnedTreeNodeA->GetNodeName() == PinnedTreeNodeB->GetNodeName();
		}
		
		if (PropertyHandleA->IsCategoryHandle() || PropertyHandleB->IsCategoryHandle())
		{
			// category nodes
			return PropertyHandleA->GetPropertyDisplayName().ToString() == PropertyHandleB->GetPropertyDisplayName().ToString();
		}
		
		return PropertyHandleA->GetProperty() == PropertyHandleB->GetProperty();
	}

	virtual bool ShouldMatchByValue(const TWeakPtr<FDetailTreeNode>& TreeNode) const override
	{
		return false;
	}

	virtual bool ShouldInheritEqualFromChildren(const TWeakPtr<FDetailTreeNode>& TreeNodeA, const TWeakPtr<FDetailTreeNode>& TreeNodeB) const override
	{
		return true;
	}
	
	const TWeakPtr<FInstanceDataObjectFixupPanel> LeftPanel;
	const TWeakPtr<FInstanceDataObjectFixupPanel> RightPanel;
};

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SInstanceDataObjectFixupTool::Construct(const FArguments& InArgs)
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");

	SelectedClassPath = InArgs._SelectedClassPath;
	StagedTransforms = InArgs._StagedTransforms;

	ChildSlot
	[
		SAssignNew(Border, SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(NoPadding)
	];
}

void SInstanceDataObjectFixupTool::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (PanelDiff != nullptr)
	{ 
		PanelDiff->Tick();
	}
}

void SInstanceDataObjectFixupTool::SetDockTab(const TSharedRef<SDockTab>& DockTab)
{
	OwningDockTab = DockTab;
}

void SInstanceDataObjectFixupTool::ResetDiff()
{
	Border->SetContent(SNullWidget::NullWidget);
	PanelDiff.Reset();
}

TObjectPtr<UObject> SInstanceDataObjectFixupTool::CreateInstanceDataObjectSnapshot()
{
	using namespace UE::DataRecoveryTool::Utils;
	using namespace UE::DataRecoveryTool::Utils::Snapshot;


	const TSharedPtr<FTopLevelAssetPath> ClassPath = SelectedClassPath.Pin();

	if (!ClassPath)
	{
		return nullptr;
	}

	TObjectPtr<UObject> Owner;

	if (GetInstanceDataObjectFromSelectedClassPath(*ClassPath, &Owner) == nullptr)
	{
		return nullptr;
	}

	// CPFUO the instance

	UObject* OwnerSnapshot = NewObject<UObject>(GetTransientPackage(), Owner->GetClass(), NAME_None, FlagsToAdd & ~FlagsToRemove);

	TMap<UObject*, UObject*> ReplacementMap;
	ReplacementMap.Add(Owner, OwnerSnapshot);

	UEngine::FCopyPropertiesForUnrelatedObjectsParams Params;

	Params.OptionalReplacementMappings = &ReplacementMap;

	Params.bPerformDuplication = true;
	Params.bOnlyHandleDirectSubObjects = false;
	Params.bReplaceInternalReferenceUponRead = true;

	UEngine::CopyPropertiesForUnrelatedObjects(Owner, OwnerSnapshot, Params);

	OwnerSnapshot->ClearFlags(FlagsToRemove);
	OwnerSnapshot->SetFlags(FlagsToAdd);

	return UE::FPropertyBagRepository::Get().FindInstanceDataObject(OwnerSnapshot);
}

TSharedRef<SWidget> SInstanceDataObjectFixupTool::MakeDiffViewPanel()
{
	const float NoPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.None");
	const float SmallPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Small");
	const float NormalPadding = FDataRecoveryToolStyle::Get().GetFloat("DataRecoveryTool.Padding.Normal");

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(NoPadding, SmallPadding)
		.FillHeight(1.f)
		[
			SNew(SScrollBox)
			.Orientation(EOrientation::Orient_Vertical)
			+ SScrollBox::Slot()
			[
				SAssignNew(Splitter, SDetailsSplitter)
				.RowHighlightColor_Static(&SInstanceDataObjectFixupTool::GetRowHighlightColor)
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(SmallPadding)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(NormalPadding)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("ResetChangesOnInstance_Tooltip", "Undo all mappings for this class type"))
				.OnClicked(this, &SInstanceDataObjectFixupTool::OnResetInstanceDataObjectSnapshot)
				.IsEnabled_Raw(this, &SInstanceDataObjectFixupTool::CanResetInstanceDataObjectSnapshot)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("GenericCommands.Undo"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(SmallPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ResetChangesOnInstance", "Reset mappings for this class"))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(NormalPadding)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("AutoMarkForDeletion_Tooltip", "Mark remaining conflicts for deletion"))
				.OnClicked(this, &SInstanceDataObjectFixupTool::OnAutoMarkForDeletion)
				.IsEnabled_Raw(this, &SInstanceDataObjectFixupTool::CanAutoMarkForDeletion)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Delete"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(SmallPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AutoMarkForDeletion", "Delete remaining conflicts"))
					]
				]
			]
		];
}

void SInstanceDataObjectFixupTool::GenerateDetailsViews()
{

#if USE_SNAPSHOT_INSTANCE
	UObject* InstanceDataObject = CreateInstanceDataObjectSnapshot();

	if (InstanceDataObject == nullptr)
	{
		ResetDiff();
		return;
	}
#else
	const TSharedPtr<FTopLevelAssetPath> ClassPath = SelectedClassPath.Pin();

	if (!ClassPath)
	{
		ResetDiff();
		return;
	}

	UObject* InstanceDataObject = UE::DataRecoveryTool::Utils::GetInstanceDataObjectFromSelectedClassPath(*ClassPath);
	if (!InstanceDataObject)
	{
		ResetDiff();
		return;
	}
#endif

	UObject* Owner = const_cast<UObject*>(UE::FPropertyBagRepository::Get().FindInstanceForDataObject(InstanceDataObject));
	UE::DataRecoveryTool::Utils::ApplyTransforms(StagedTransforms, InstanceDataObject, Owner);

	constexpr int32 LeftIndex = 0;
	constexpr int32 RightIndex = 1;

	Panels[LeftIndex] = MakeShared<FInstanceDataObjectFixupPanel>(InstanceDataObject, FInstanceDataObjectFixupPanel::EViewFlags::DefaultLeftPanel, StagedTransforms);
	Panels[RightIndex] = MakeShared<FInstanceDataObjectFixupPanel>(InstanceDataObject, FInstanceDataObjectFixupPanel::EViewFlags::DefaultRightPanel, StagedTransforms);

	// generate the panels
	Panels[LeftIndex]->GenerateDetailsView(true);
	Panels[RightIndex]->GenerateDetailsView(false);

	// diff the panels
	PanelDiff = MakeShared<FAsyncDetailViewDiff>(
		Panels[LeftIndex]->DetailsView.ToSharedRef(), Panels[RightIndex]->DetailsView.ToSharedRef());
	PanelDiff->SetDiffSpecification<FInstanceDataObjectFixupSpecification>(Panels[LeftIndex], Panels[RightIndex]);

	SLinkableScrollBar::LinkScrollBars(Panels[LeftIndex]->LinkableScrollBar.ToSharedRef(), Panels[RightIndex]->LinkableScrollBar.ToSharedRef(),
		TAttribute<TArray<FVector2f>>::CreateRaw(PanelDiff.Get(), &FAsyncDetailViewDiff::GenerateScrollSyncRate)); // TODO: Make work w/ CreateShared
	Panels[LeftIndex]->SetDiffAgainstRight(PanelDiff);
	Panels[RightIndex]->SetDiffAgainstLeft(PanelDiff);


	Border->SetContent(
		MakeDiffViewPanel()
	);

	// add the panels to the splitter
	for (TSharedPtr Panel : Panels)
	{
		Splitter->AddSlot(
			SDetailsSplitter::Slot()
			.DetailsView(Panel->DetailsView.ToSharedRef())
			.DifferencesWithRightPanel(Panel.Get(), &FInstanceDataObjectFixupPanel::GetDiffAgainstRight)
			.IsReadonly_Lambda([Panel = Panel.ToWeakPtr()]() { return Panel.IsValid() ?
				Panel.Pin()->HasViewFlag(FInstanceDataObjectFixupPanel::EViewFlags::ReadonlyValues) : true; })
		);
	}
}

	

FLinearColor SInstanceDataObjectFixupTool::GetRowHighlightColor(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode)
{
	switch (DiffNode->DiffResult)
	{
	case ETreeDiffResult::MissingFromTree1:
		// adds are green
		return FLinearColor(0.f, 1.f, 0.f, .5f);
	case ETreeDiffResult::MissingFromTree2:
		// removes/conflicts are over-saturated red
		return FLinearColor(1.5f, 0.3f, 0.3f, 1.f);
	case ETreeDiffResult::DifferentValues:
		return FLinearColor(0.f, 1.f, 1.f, .8f);
	case ETreeDiffResult::Identical:
		return FLinearColor();
	default:
		check(false);
		return FLinearColor();
	}
}

bool SInstanceDataObjectFixupTool::IsResolved() const
{
	for (const TSharedPtr<FInstanceDataObjectFixupPanel>& Panel : Panels)
	{
		if (Panel == nullptr)
		{
			continue;
		}

		if (!Panel->AreAllConflictsRedirected())
		{
			return false;
		}
	}
	return true;
}

FReply SInstanceDataObjectFixupTool::OnAutoMarkForDeletion() const
{
	for (const TSharedPtr<FInstanceDataObjectFixupPanel>& Panel : Panels)
	{
		Panel->AutoApplyMarkDeletedActions();
	}
	return FReply::Handled();
}

bool SInstanceDataObjectFixupTool::CanAutoMarkForDeletion() const
{
	return !IsResolved();
}

FReply SInstanceDataObjectFixupTool::OnResetInstanceDataObjectSnapshot()
{
	TSharedPtr<FTopLevelAssetPath> ClassPath = SelectedClassPath.Pin();
	if (ClassPath == nullptr)
	{
		return FReply::Unhandled();
	}

	if (const TSharedPtr Transforms = StagedTransforms.Pin())
	{
		for (const TPair<FTopLevelAssetPath, UE::FInstanceDataTransformSet>& TransformPair : *Transforms)
		{
			if (FTopLevelAssetPath TransformPath = TransformPair.Key;
				UE::DataRecoveryTool::Utils::ContainsPath(*ClassPath, TransformPath))
			{
				if (UE::FInstanceDataTransformSet* TransformSet = Transforms->Find(TransformPath))
				{
					if (UStruct* LoadedStruct = Cast<UStruct>(StaticLoadObject(UStruct::StaticClass(), nullptr, TransformPath.ToString())))
					{
						*TransformSet = UE::FInstanceDataTransforms::Get().GetTransformSet(*LoadedStruct);
					}
					else
					{
						return FReply::Unhandled();
					}
				}
				else
				{
					return FReply::Unhandled();
				}
			}
		}
	}

	GenerateDetailsViews();

	return FReply::Handled();

}

bool SInstanceDataObjectFixupTool::CanResetInstanceDataObjectSnapshot() const
{
	TSharedPtr<FTopLevelAssetPath> ClassPath = SelectedClassPath.Pin();
	if (ClassPath == nullptr)
	{
		return false;
	}

	if (const TSharedPtr Transforms = StagedTransforms.Pin())
	{
		for (const TPair<FTopLevelAssetPath, UE::FInstanceDataTransformSet>& TransformPair : *Transforms)
		{
			if (FTopLevelAssetPath TransformPath = TransformPair.Key;
				UE::DataRecoveryTool::Utils::ContainsPath(*ClassPath, TransformPath))
			{
				UE::FInstanceDataTransformSet* TransformSet = Transforms->Find(TransformPath);
				if (TransformSet && TransformSet->Operations.Num() > 0)
				{
					return true;
				}
			}
		}
	}

	return false;
}

UE::FInstanceDataTransformSet* SInstanceDataObjectFixupTool::GetSelectedStagedTransforms() const
{
	if (const TSharedPtr<FTopLevelAssetPath> SelectedPath = SelectedClassPath.Pin())
	{
		if (const TSharedPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>>& Transforms = StagedTransforms.Pin())
		{
			if (UE::FInstanceDataTransformSet* Found = Transforms->Find(*SelectedPath))
			{
				return Found;
			}
		}
	}
	return nullptr;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE

#undef USE_SNAPSHOT_INSTANCE