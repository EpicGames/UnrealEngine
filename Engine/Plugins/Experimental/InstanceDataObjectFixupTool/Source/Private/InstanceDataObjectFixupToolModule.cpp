// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataObjectFixupToolModule.h"

#include "DataRecoveryTool.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "InstanceDataObjectFixupToolModule"

FName DataRecoveryToolDialogName = TEXT("DataRecoveryToolDialog");

void FInstanceDataObjectFixupToolModule::StartupModule()
{

}


void FInstanceDataObjectFixupToolModule::RegisterTabSpawners(const TOptional<FTopLevelAssetPath>& ClassPath) const
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		DataRecoveryToolDialogName,
		FOnSpawnTab::CreateLambda([this, ClassPath](const FSpawnTabArgs& Args)
			{
				return CreateDataRecoveryToolTab(ClassPath);
			}))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetDisplayName(LOCTEXT("DataRecoveryTool_DisplayName", "Data Recovery Tool"))
		.SetTooltipText(LOCTEXT("DataRecoveryTool_DisplayNameToolTip", "Opens the Data recovery Tool"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"));
}

void FInstanceDataObjectFixupToolModule::UnregisterTabSpawners() const
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DataRecoveryToolDialogName);
	}
}

void FInstanceDataObjectFixupToolModule::ShutdownModule()
{
	UnregisterTabSpawners();
}

TSharedRef<SDockTab> FInstanceDataObjectFixupToolModule::CreateDataRecoveryToolTab(const TOptional<FTopLevelAssetPath>& ClassPath) const
{
	const TSharedRef<SDataRecoveryTool> DataRecoveryTool = SNew(SDataRecoveryTool)
		.SelectedClassPath(ClassPath);

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			DataRecoveryTool
		];

	DataRecoveryTool->SetDockTab(DockTab);

	return DockTab;
}

bool FInstanceDataObjectFixupToolModule::IsDataRecoveryToolEnabled() const
{
	if (const IConsoleVariable* CVarEnableDataRecoveryTool = IConsoleManager::Get().FindConsoleVariable(TEXT("IDO.EnableDataRecoveryTool")))
	{
		return CVarEnableDataRecoveryTool->GetBool();
	}

	return false;
}

void FInstanceDataObjectFixupToolModule::CreateDataRecoveryToolDialog(const TOptional<FTopLevelAssetPath>& ClassPath) const
{
	if (!IsDataRecoveryToolEnabled())
	{
		return;
	}

	RegisterTabSpawners(ClassPath);

	if (const TSharedPtr<SDockTab> DockTab = FGlobalTabmanager::Get()->TryInvokeTab(DataRecoveryToolDialogName))
	{
		DockTab->DrawAttention();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FInstanceDataObjectFixupToolModule, InstanceDataObjectFixupTool)
