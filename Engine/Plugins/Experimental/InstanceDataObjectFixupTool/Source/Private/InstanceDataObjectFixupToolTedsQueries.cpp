// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceDataObjectFixupToolTedsQueries.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/World.h"
#include "UObject/PropertyBagRepository.h"
#include "InstanceDataObjectFixupToolModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "TedsAlerts.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstanceDataObjectFixupToolTedsQueries)

#define LOCTEXT_NAMESPACE "FixupToolTedsQueries"

void UInstanceDataObjectFixupToolTedsQueryFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Add fix-up tool to serialization placeholder alerts"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)),
			[](IQueryContext& Context, RowHandle Row)
			{
				static FName AlertName("PlaceholderAlert");
				Alerts::UpdateAlertAction(Context, Row, AlertName, ShowFixUpToolForPlaceholders);
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementUObjectColumn, FTypedElementPropertyBagPlaceholderTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add fix-up tool to serialization loose property alerts"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)),
			[](IQueryContext& Context, RowHandle Row)
			{
				static FName AlertName = FName("EntityLoosePropertiesErrorAlert");
				Alerts::UpdateAlertAction(Context, Row, AlertName, ShowFixUpToolForLooseProperties);
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementUObjectColumn, FTypedElementLoosePropertyTag>()
		.Compile());
}

void UInstanceDataObjectFixupToolTedsQueryFactory::ShowFixUpToolForPlaceholders(UE::Editor::DataStorage::RowHandle Row)
{
	FNotificationInfo Info(LOCTEXT("PlaceholderResolutionSuggestion", "Please fix your Verse code and/or rename the Verse class back to the original name."));
	
	Info.ExpireDuration = 4.0f;
	FSlateNotificationManager::Get().AddNotification(Info);
}

void UInstanceDataObjectFixupToolTedsQueryFactory::ShowFixUpToolForLooseProperties(UE::Editor::DataStorage::RowHandle Row)
{
	ShowFixUpTool(Row, true);
}

void UInstanceDataObjectFixupToolTedsQueryFactory::ShowFixUpTool(UE::Editor::DataStorage::RowHandle Row, bool bRecurseIntoObject)
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(Row))
	{
		if (bRecurseIntoObject)
		{
			UObject* Owner = ObjectColumn->Object.Get();
			TOptional<FTopLevelAssetPath> ClassPath;

			UE::FPropertyBagRepository::Get().FindNestedInstanceDataObject(Owner, true,
				[Owner, &ClassPath](UObject* NestedObject)
				{
					ClassPath = Owner->GetClass()->GetClassPathName();
				});


			FInstanceDataObjectFixupToolModule::Get().CreateDataRecoveryToolDialog(ClassPath);
			
		}
		else
		{
			UObject* Owner = ObjectColumn->Object.Get();
			if (TObjectPtr<UObject> InstanceDataObject = UE::FPropertyBagRepository::Get().FindInstanceDataObject(Owner))
			{
				FTopLevelAssetPath ClassPath = Owner->GetClass()->GetClassPathName();
				FInstanceDataObjectFixupToolModule::Get().CreateDataRecoveryToolDialog(ClassPath);
			}
		}
	}
}
#undef LOCTEXT_NAMESPACE
