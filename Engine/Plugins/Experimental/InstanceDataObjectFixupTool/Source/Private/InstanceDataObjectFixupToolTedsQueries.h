// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "InstanceDataObjectFixupToolTedsQueries.generated.h"

UCLASS()
class UInstanceDataObjectFixupToolTedsQueryFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UInstanceDataObjectFixupToolTedsQueryFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	static void ShowFixUpToolForPlaceholders(UE::Editor::DataStorage::RowHandle Row);
	static void ShowFixUpToolForLooseProperties(UE::Editor::DataStorage::RowHandle Row);
	static void ShowFixUpTool(UE::Editor::DataStorage::RowHandle Row, bool bRecurseIntoObject);
};
