// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TedsTableViewer/Public/TedsTableViewerModel.h"
#include "TedsTableViewer/Public/Widgets/STedsTableViewer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCompoundWidget.h"
#include "DataRecoveryToolUtils.h"
#include "UObject/InstanceDataTransforms.h"


#define UE_API INSTANCEDATAOBJECTFIXUPTOOL_API


class SInstanceDataObjectFixupTool;

namespace UE
{
	namespace Editor::DataStorage::QueryStack
	{
		class FRowQueryResultsNode;
	}

	class FRowQueryResultsNode;
	class FInstanceDataTransformSet;
}

class SDataRecoveryTool : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataRecoveryTool)
	{}
		SLATE_ARGUMENT(TOptional<FTopLevelAssetPath>, SelectedClassPath)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API void SetDockTab(const TSharedRef<SDockTab>& DockTab);

private:

	UE_API TSharedRef<SWidget> MakeTopBar();

	UE_API TSharedRef<SWidget> MakePanel(const FText& InTitle, const TSharedRef<SWidget>& InContent,
		const TOptional<FOnGetContent>& InMenuContent = TOptional<FOnGetContent>{});

	UE_API TSharedRef<SWidget> MakeClassesPanel();
	UE_API TSharedRef<SWidget> MakeTEDSTableViewer();
	UE_API void OnListSelectionChanged(UE::Editor::DataStorage::RowHandle RowItem, ESelectInfo::Type Info);

	void SetSelectedClass();

	UE_API TSharedRef<SWidget> MakeRightPanel();

	UE_API TSharedRef<SWidget> MakeDiffSection();

	FReply OnApplyClicked() const;
	FReply OnSaveAndApplyClicked() const;

	TWeakPtr<SDockTab> OwningDockTab;

	TSharedPtr<FTopLevelAssetPath> SelectedClassPath;

	TSharedPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>> StagedTransforms;

	TSharedPtr<UE::Editor::DataStorage::STedsTableViewer> ClassesTableViewer;

	TSharedPtr<UE::Editor::DataStorage::QueryStack::IRowNode> ReferenceQueryResultsNode;

	TSharedPtr<SInstanceDataObjectFixupTool> DiffView;
};

#undef UE_API
