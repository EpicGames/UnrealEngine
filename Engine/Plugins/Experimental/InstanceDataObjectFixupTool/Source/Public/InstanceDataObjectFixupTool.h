// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AsyncDetailViewDiff.h"
#include "UObject/InstanceDataTransforms.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API INSTANCEDATAOBJECTFIXUPTOOL_API

class SDetailsSplitter;
class FInstanceDataObjectFixupPanel;
struct FPropertySoftPath;
class IStructureDetailsView;
namespace UE
{
	class FInstanceDataTransformSet;
}

/**
 * This tool diffs multiple property bags of one format against the same number of property bags of another format.
 * 
 */
class SInstanceDataObjectFixupTool : public SCompoundWidget
{
public:
	using FStagedTransformMap = TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>;

	SLATE_BEGIN_ARGS(SInstanceDataObjectFixupTool)
	{}
		SLATE_ARGUMENT(TWeakPtr<FTopLevelAssetPath>, SelectedClassPath)
		SLATE_ARGUMENT(TWeakPtr<FStagedTransformMap>, StagedTransforms)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	UE_API void SetDockTab(const TSharedRef<SDockTab>& DockTab);
	UE_API void GenerateDetailsViews();
	static UE_API FLinearColor GetRowHighlightColor(const TUniquePtr<FAsyncDetailViewDiff::DiffNodeType>& DiffNode);
	UE_API bool IsResolved() const;
	
private:
	void ResetDiff();
	TObjectPtr<UObject> CreateInstanceDataObjectSnapshot();

	FReply OnAutoMarkForDeletion() const;
	bool CanAutoMarkForDeletion() const;

	TSharedRef<SWidget> MakeDiffViewPanel();

	FReply OnResetInstanceDataObjectSnapshot();
	bool CanResetInstanceDataObjectSnapshot() const;

	UE::FInstanceDataTransformSet* GetSelectedStagedTransforms() const;

	TWeakPtr<SDockTab> OwningDockTab;
	TStaticArray<TSharedPtr<FInstanceDataObjectFixupPanel>, 2> Panels;
	TSharedPtr<FAsyncDetailViewDiff> PanelDiff;

	TSharedPtr<SDetailsSplitter> Splitter;

	TWeakPtr<TMap<FTopLevelAssetPath, UE::FInstanceDataTransformSet>> StagedTransforms;

	TSharedPtr<SBorder> Border;
	TWeakPtr<FTopLevelAssetPath> SelectedClassPath;

	
};

#undef UE_API
