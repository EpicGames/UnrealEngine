// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API INSTANCEDATAOBJECTFIXUPTOOL_API

class FInstanceDataObjectFixupToolModule : public FDefaultModuleImpl
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	static FInstanceDataObjectFixupToolModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FInstanceDataObjectFixupToolModule>("InstanceDataObjectFixupTool");
	}

	UE_API void CreateDataRecoveryToolDialog(const TOptional<FTopLevelAssetPath>& ClassPath) const;

	UE_API void RegisterTabSpawners(const TOptional<FTopLevelAssetPath>& ClassPath = {}) const;
	UE_API void UnregisterTabSpawners() const;

private:
	TSharedRef<SDockTab> CreateDataRecoveryToolTab(const TOptional<FTopLevelAssetPath>& ClassPath = {}) const;

	bool IsDataRecoveryToolEnabled() const;
};

#undef UE_API
